[DerivedDataCacheGraphs]
TestInvalidCacheGraph1=Invalid
TestInvalidCacheGraph2="Invalid
TestInvalidCacheGraph3="Invalid\
TestInvalidCacheGraph4="Invalid\"
TestInvalidCacheGraph5=(Invalid
TestInvalidCacheGraph6=(Invalid,
TestInvalidCacheStore1=(TestInvalidStore1)
TestInvalidCacheStore2=(TestInvalidStore1(Key1=Value1))
TestInvalidCacheStore3=(TestInvalidStore2(Key1=Value1))
TestCacheGraphInlineStore1 = ( TestInlineStore ( Key1 = Value1 ( A = 1 , B = 2, C ) , Key2 = Value2, Key3 = D (123) ) ) ; Comment
TestCacheGraphInlineStore2 = ( TestInlineStore = TestMissing ( Key1 = Value1 ( A = 1 , B = 2, C ) , Key2 = Value2, Key3 = D (123) ) ) ; Comment
TestCacheGraph0=()
TestCacheGraph1=(TestStore1)
TestCacheGraph2=(TestStore1,TestStore2,TestStore3)
TestCacheGraph3=(TestStore1,TestStore2,TestStore3())
TestCacheGraph4 = ( TestStore1 , TestStore2 , TestStore3 ) ; Comment
TestCacheGraph5 = ( TestStore1 , TestStore2 ( ) , TestStore3 ( Key1 = Value1 , Key2 = Value2 ) ) ; Comment
TestCacheGraph6 = ( TestStore1 = ( Key1 = Value1 ) ) ; Comment
TestCacheGraph7 = ( TestStore1 = TestStore5 ( Key1 = Value1 ) ) ; Comment
TestCacheGraph8 = ( TestStore1 = TestStore6 ) ; Comment
TestCacheGraph9 = ( TestStore1 ( Key1 = Value1 ) , Deprecated = "Use TestCacheGraph1." ) ; Comment

[DerivedDataCacheStores]
TestInvalidStore1=Invalid
TestInvalidStore2=(Base=TestInvalidStore1,Key2=Value2)
TestInvalidStore3=(Base=TestStore1,Base=TestStore2)
TestStore1=(Key1=Store1Value,Key2=Store1Value,ValueOnly,(X="A=1, \"B\"=2, C=3"))
TestStore2=(Base=TestStore1,Key1=Store2Value)
TestStore3 = ( ValueOnly , Key1 = Store3Value , Key2 = Store3Value , Key3 = ( A = 1 , B = 2 , C ) ) ; Comment
TestStore4 = ( Key1 = Store4Value , Base = TestStore2 ) ; Comment
TestStore5 = ( Key1 = Store5Value , Key2 = Store5Value ) ; Comment
TestStore6 = ( Base = TestStore1 ) ; Comment
TestStore7 = ( Base = TestStore1 , Deprecated = "Use TestStore1." ) ; Comment
TestStore8 = ( Type = Cloud ) ; Comment

[TestLegacyCacheGraph1]
Root=(Type=KeyLength, Length=120, Inner=Verify)
Verify=(Type=Verify, Inner=AsyncPut)
AsyncPut=(Type=AsyncPut, Inner=Hierarchy)
Hierarchy=(Type=Hierarchical, Inner=TestLegacyStore1, Inner=TestStore2, Inner=TestStore8)
TestLegacyStore1=(Type=Zen, Key1=LegacyStore1Value, Key2=LegacyStore1Value)
TestStore2=(Type=Zen, Key1=LegacyStore2Value)
